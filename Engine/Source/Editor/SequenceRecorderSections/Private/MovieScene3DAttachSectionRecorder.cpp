// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene3DAttachSectionRecorder.h"

#include "ISequenceRecorder.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneObjectBindingID.h"
#include "Sections/MovieScene3DAttachSection.h"
#include "SequenceRecorderUtils.h"
#include "Templates/Casts.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

TSharedPtr<IMovieSceneSectionRecorder> FMovieScene3DAttachSectionRecorderFactory::CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const
{
	return MakeShareable(new FMovieScene3DAttachSectionRecorder);
}

bool FMovieScene3DAttachSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>();
}

void FMovieScene3DAttachSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& Guid, float Time)
{
	ObjectGuid = Guid;
	ActorToRecord = CastChecked<AActor>(InObjectToRecord);
	MovieScene = InMovieScene;
	TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
}

void FMovieScene3DAttachSectionRecorder::FinalizeSection(float CurrentTime)
{
}

void FMovieScene3DAttachSectionRecorder::Record(float CurrentTime)
{
	if(ActorToRecord.IsValid())
	{
		if(MovieSceneSection.IsValid())
		{
			FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber CurrentFrame = (CurrentTime * TickResolution).FloorToFrame();

			MovieSceneSection->ExpandToFrame(CurrentFrame);
		}

		// get attachment and check if the actor we are attached to is being recorded
		FName SocketName;
		FName ComponentName;
		AActor* AttachedToActor = SequenceRecorderUtils::GetAttachment(ActorToRecord.Get(), SocketName, ComponentName);

		ISequenceRecorder& SequenceRecorder = FModuleManager::GetModuleChecked<ISequenceRecorder>("SequenceRecorder");
		FGuid Guid = SequenceRecorder.GetRecordingGuid(AttachedToActor);
		if(AttachedToActor && Guid.IsValid())
		{
			// create the track if we haven't already
			if(!AttachTrack.IsValid())
			{
				AttachTrack = MovieScene->AddTrack<UMovieScene3DAttachTrack>(ObjectGuid);
			}

			// check if we need a section or if the actor we are attached to has changed
			if(!MovieSceneSection.IsValid() || AttachedToActor != ActorAttachedTo.Get())
			{
				MovieSceneSection = Cast<UMovieScene3DAttachSection>(AttachTrack->CreateNewSection());
				AttachTrack->AddSection(*MovieSceneSection);

				FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
				FFrameNumber CurrentFrame = (CurrentTime * TickResolution).FloorToFrame();

				MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));
				MovieSceneSection->SetConstraintBindingID(UE::MovieScene::FRelativeObjectBindingID(Guid));
				MovieSceneSection->AttachSocketName = SocketName;
				MovieSceneSection->AttachComponentName = ComponentName;

				MovieSceneSection->TimecodeSource = TimecodeSource;
			}

			ActorAttachedTo = AttachedToActor;
		}
		else
		{
			// no attachment, so end the section recording if we have any
			MovieSceneSection = nullptr;
		}
	}
}
