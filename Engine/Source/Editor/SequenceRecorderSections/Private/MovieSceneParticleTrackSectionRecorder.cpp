// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneParticleTrackSectionRecorder.h"
#include "Tracks/MovieSceneParticleTrack.h"
#include "MovieScene.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "SequenceRecorderUtils.h"

TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneParticleTrackSectionRecorderFactory::CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const
{
	return MakeShareable(new FMovieSceneParticleTrackSectionRecorder);
}

bool FMovieSceneParticleTrackSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<UParticleSystemComponent>();
}

FMovieSceneParticleTrackSectionRecorder::~FMovieSceneParticleTrackSectionRecorder()
{
	if(DelegateProxy.IsValid())
	{
		DelegateProxy->SectionRecorder = nullptr;
		DelegateProxy->RemoveFromRoot();
		DelegateProxy.Reset();
	}
}

void FMovieSceneParticleTrackSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* MovieScene, const FGuid& Guid, float Time)
{
	SystemToRecord = CastChecked<UParticleSystemComponent>(InObjectToRecord);

	UMovieSceneParticleTrack* ParticleTrack = MovieScene->FindTrack<UMovieSceneParticleTrack>(Guid);
	if (!ParticleTrack)
	{
		ParticleTrack = MovieScene->AddTrack<UMovieSceneParticleTrack>(Guid);
	}
	else
	{
		ParticleTrack->RemoveAllAnimationData();
	}

	if(ParticleTrack)
	{
		MovieSceneSection = Cast<UMovieSceneParticleSection>(ParticleTrack->CreateNewSection());

		ParticleTrack->AddSection(*MovieSceneSection);

		FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = (Time * TickResolution).FloorToFrame();
		MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		bWasTriggered = false;

		DelegateProxy = NewObject<UMovieSceneParticleTrackSectionRecorder>();
		DelegateProxy->SectionRecorder = this;
		DelegateProxy->AddToRoot();
		UParticleSystemComponent::OnSystemPreActivationChange.AddUObject(DelegateProxy.Get(), &UMovieSceneParticleTrackSectionRecorder::OnTriggered);
	}

	PreviousState = EParticleKey::Deactivate;
}

void FMovieSceneParticleTrackSectionRecorder::FinalizeSection(float CurrentTime)
{
}

void FMovieSceneParticleTrackSectionRecorder::Record(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if(SystemToRecord.IsValid())
	{
		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber KeyTime         = (CurrentTime * TickResolution).FloorToFrame();

		MovieSceneSection->ExpandToFrame(KeyTime);

		EParticleKey NewState = EParticleKey::Deactivate;
		if(SystemToRecord->IsRegistered() && SystemToRecord->IsActive() && !SystemToRecord->bWasDeactivated)
		{
			if(bWasTriggered)
			{
				NewState = EParticleKey::Trigger;
				bWasTriggered = false;
			}
			else
			{
				NewState = EParticleKey::Activate;
			}
		}
		else
		{
			NewState = EParticleKey::Deactivate;
		}

		if(NewState != PreviousState)
		{
			FMovieSceneParticleChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneParticleChannel>(0);
			if (ensure(Channel))
			{
				Channel->GetData().AddKey(KeyTime, (uint8)NewState);
			}
		}

		if(NewState == EParticleKey::Trigger)
		{
			NewState = EParticleKey::Activate;
		}
		PreviousState = NewState;
	}
}

void UMovieSceneParticleTrackSectionRecorder::OnTriggered(UParticleSystemComponent* Component, bool bActivating)
{ 
	if(SectionRecorder && SectionRecorder->SystemToRecord.Get() == Component)
	{
		SectionRecorder->bWasTriggered = bActivating;
	}
}
