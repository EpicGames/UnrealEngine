// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSpawnSectionRecorder.h"

#include "Animation/AnimationRecordingSettings.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneBoolSection.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "Templates/Casts.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "UObject/UObjectGlobals.h"

TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneSpawnSectionRecorderFactory::CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const
{
	return MakeShareable(new FMovieSceneSpawnSectionRecorder);
}

bool FMovieSceneSpawnSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>();
}

void FMovieSceneSpawnSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* MovieScene, const FGuid& Guid, float Time)
{
	if (MovieScene->FindPossessable(Guid))
	{
		return;
	}

	ObjectToRecord = InObjectToRecord;

	UMovieSceneSection* NewMovieSceneSection = nullptr;

	UMovieSceneSpawnTrack* SpawnTrack = MovieScene->FindTrack<UMovieSceneSpawnTrack>(Guid);
	if (!SpawnTrack)
	{
		SpawnTrack = MovieScene->AddTrack<UMovieSceneSpawnTrack>(Guid);
	}
	else
	{
		SpawnTrack->RemoveAllAnimationData();
	}

	if(SpawnTrack)
	{
		MovieSceneSection = Cast<UMovieSceneBoolSection>(SpawnTrack->CreateNewSection());

		SpawnTrack->AddSection(*MovieSceneSection);
		SpawnTrack->SetObjectId(Guid);

		FMovieSceneBoolChannel* BoolChannel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		check(BoolChannel);
		BoolChannel->SetDefault(false);
		BoolChannel->GetData().AddKey(0, false);

		FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = (Time * TickResolution).FloorToFrame();
		MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
	}

	bWasSpawned = false;
}

void FMovieSceneSpawnSectionRecorder::FinalizeSection(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	const bool bSpawned = ObjectToRecord.IsValid();
	if(bSpawned != bWasSpawned)
	{
		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel) && MovieSceneSection->HasEndFrame())
		{
			Channel->GetData().AddKey(MovieSceneSection->GetExclusiveEndFrame()-1, bSpawned);
		}
	}

	// If the section is degenerate, assume the actor was spawned and destroyed. Give it a 1 frame spawn section.
	if (MovieSceneSection->GetRange().IsDegenerate() && MovieSceneSection->HasEndFrame())
	{
		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel))
		{
			double       OneFrameInterval = GetDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.SampleFrameRate.AsInterval();

			FFrameRate   TickResolution   = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber StartTime        = MovieSceneSection->GetExclusiveEndFrame() - (OneFrameInterval * TickResolution).CeilToFrame();

			Channel->GetData().AddKey(StartTime, true);
			MovieSceneSection->SetStartFrame(StartTime);
		}
	}
}

void FMovieSceneSpawnSectionRecorder::Record(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if(ObjectToRecord.IsValid())
	{
		FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		MovieSceneSection->ExpandToFrame((CurrentTime * TickResolution).FloorToFrame());
	}

	const bool bSpawned = ObjectToRecord.IsValid();
	if(bSpawned != bWasSpawned)
	{
		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel))
		{
			FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber KeyTime         = (CurrentTime * TickResolution).FloorToFrame();

			Channel->GetData().UpdateOrAddKey(KeyTime, bSpawned);
		}
	}
	bWasSpawned = bSpawned;
}
