// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneVisibilitySectionRecorder.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "KeyParams.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Tracks/MovieSceneVisibilityTrack.h"
#include "MovieScene.h"
#include "ActorRecordingSettings.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTimeHelpers.h"
#include "SequenceRecorderUtils.h"

static const FName ActorVisibilityTrackName = TEXT("bHidden");
static const FName ComponentVisibilityTrackName = TEXT("bHiddenInGame");

TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneVisibilitySectionRecorderFactory::CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const
{
	UMovieSceneVisibilitySectionRecorderSettings* Settings = InActorRecordingSettings.GetSettingsObject<UMovieSceneVisibilitySectionRecorderSettings>();
	check(Settings);
	if (Settings->bRecordVisibility)
	{
		return MakeShareable(new FMovieSceneVisibilitySectionRecorder);
	}

	return nullptr;
}

bool FMovieSceneVisibilitySectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<AActor>() || InObjectToRecord->IsA<USceneComponent>();
}

void FMovieSceneVisibilitySectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* MovieScene, const FGuid& Guid, float Time)
{
	ObjectToRecord = InObjectToRecord;

	UMovieSceneVisibilityTrack* VisibilityTrack = MovieScene->FindTrack<UMovieSceneVisibilityTrack>(Guid);
	if (!VisibilityTrack)
	{
		VisibilityTrack = MovieScene->AddTrack<UMovieSceneVisibilityTrack>(Guid);
	}
	else
	{
		VisibilityTrack->RemoveAllAnimationData();
	}

	if(VisibilityTrack)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get());

		if (SceneComponent)
		{
			VisibilityTrack->SetPropertyNameAndPath(ComponentVisibilityTrackName, ComponentVisibilityTrackName.ToString());
		}
		else
		{
			VisibilityTrack->SetPropertyNameAndPath(ActorVisibilityTrackName, ActorVisibilityTrackName.ToString());
		}

		MovieSceneSection = Cast<UMovieSceneBoolSection>(VisibilityTrack->CreateNewSection());

		VisibilityTrack->AddSection(*MovieSceneSection);

		FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
		if (ensure(Channel))
		{
			Channel->SetDefault(false);
		}

		bWasVisible = false;
		if(SceneComponent)
		{
			bWasVisible = SceneComponent->IsVisible() && SceneComponent->IsRegistered();
		}
		else if(AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
		{
			bWasVisible = !Actor->IsHidden();
		}

		FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (Time * TickResolution).FloorToFrame();

		if (ensure(Channel))
		{
			// if time is not at the very start of the movie scene, make sure 
			// we are hidden by default as the track will extrapolate backwards and show 
			// objects that shouldnt be visible.
			FFrameNumber LowerBoundValue = MovieScene->GetPlaybackRange().GetLowerBoundValue();
			if (CurrentFrame != LowerBoundValue)
			{
				Channel->GetData().AddKey(LowerBoundValue, false);
			}

			Channel->GetData().AddKey(CurrentFrame, bWasVisible);
		}

		MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
	}
}

void FMovieSceneVisibilitySectionRecorder::FinalizeSection(float CurrentTime)
{
}

void FMovieSceneVisibilitySectionRecorder::Record(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if(ObjectToRecord.IsValid())
	{
		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (CurrentTime * TickResolution).FloorToFrame();

		MovieSceneSection->ExpandToFrame(CurrentFrame);

		bool bVisible = false;
		if(USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
		{
			bVisible = SceneComponent->IsVisible() && SceneComponent->IsRegistered();
		}
		else if(AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
		{
			bVisible = !Actor->IsHidden();
		}

		if(bVisible != bWasVisible)
		{
			FMovieSceneBoolChannel* Channel = MovieSceneSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
			if (ensure(Channel))
			{
				Channel->GetData().AddKey(CurrentFrame, bVisible);
			}
		}
		bWasVisible = bVisible;
	}
}
