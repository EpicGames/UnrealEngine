// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "MovieScene3DAttachSectionRecorder.h"
#include "MovieSceneParticleTrackSectionRecorder.h"
#include "MovieSceneSpawnSectionRecorder.h"
#include "MovieSceneVisibilitySectionRecorder.h"

static const FName MovieSceneSectionRecorderFactoryName("MovieSceneSectionRecorderFactory");

class FSequenceRecorderSectionsModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		// register built-in recorders
		IModularFeatures::Get().RegisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieScene3DAttachSectionRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneParticleTrackSectionRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneSpawnSectionRecorder);
		IModularFeatures::Get().RegisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneVisibilitySectionRecorder);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieScene3DAttachSectionRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneParticleTrackSectionRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneSpawnSectionRecorder);
		IModularFeatures::Get().UnregisterModularFeature(MovieSceneSectionRecorderFactoryName, &MovieSceneVisibilitySectionRecorder);
	}

	FMovieScene3DAttachSectionRecorderFactory MovieScene3DAttachSectionRecorder;
	FMovieSceneParticleTrackSectionRecorderFactory MovieSceneParticleTrackSectionRecorder;
	FMovieSceneSpawnSectionRecorderFactory MovieSceneSpawnSectionRecorder;
	FMovieSceneVisibilitySectionRecorderFactory MovieSceneVisibilitySectionRecorder;
};

IMPLEMENT_MODULE(FSequenceRecorderSectionsModule, SequenceRecorderSections)
