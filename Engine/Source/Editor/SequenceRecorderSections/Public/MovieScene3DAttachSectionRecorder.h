// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieScene3DAttachSection;
class UMovieScene3DAttachTrack;
class UObject;

class SEQUENCERECORDERSECTIONS_API FMovieScene3DAttachSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieScene3DAttachSectionRecorderFactory() {}

	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const override;
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
};

class FMovieScene3DAttachSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	virtual ~FMovieScene3DAttachSectionRecorder() {}

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ActorToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ActorToRecord.Get();
	}

private:
	/** Object to record from */
	TWeakObjectPtr<class AActor> ActorToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieScene3DAttachSection> MovieSceneSection;

	/** Track we are recording to */
	TWeakObjectPtr<class UMovieScene3DAttachTrack> AttachTrack;

	/** Movie scene we are recording to */
	TWeakObjectPtr<class UMovieScene> MovieScene;

	/** Track the actor we are attached to */
	TWeakObjectPtr<class AActor> ActorAttachedTo;

	/** Identifier of the object we are recording */
	FGuid ObjectGuid;

	/** The timecode source at the beginning of recording */
	FMovieSceneTimecodeSource TimecodeSource;
};
