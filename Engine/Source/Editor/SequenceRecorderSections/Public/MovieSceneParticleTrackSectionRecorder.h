// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "Particles/ParticleSystemComponent.h"
#include "Sections/MovieSceneParticleSection.h"
#include "MovieSceneParticleTrackSectionRecorder.generated.h"

struct FMovieSceneParticleTrackSectionRecorder;

class FMovieSceneParticleTrackSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneParticleTrackSectionRecorderFactory() {}

	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const override;
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
};

UCLASS(MinimalAPI, NotBlueprintable)
class UMovieSceneParticleTrackSectionRecorder : public UObject
{
	GENERATED_BODY()

public:
	UMovieSceneParticleTrackSectionRecorder()
		: SectionRecorder(nullptr)
	{
	}

	UFUNCTION()
	void OnTriggered(UParticleSystemComponent* Component, bool bActivating);

	FMovieSceneParticleTrackSectionRecorder* SectionRecorder;
};

struct FMovieSceneParticleTrackSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	friend class UMovieSceneParticleTrackSectionRecorder;

	FMovieSceneParticleTrackSectionRecorder()
		: SystemToRecord(nullptr)
		, MovieSceneSection(nullptr)
		, bWasTriggered(false)
		, PreviousState(EParticleKey::Activate)
		, DelegateProxy(nullptr)
	{
	
	}

	virtual ~FMovieSceneParticleTrackSectionRecorder();

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* MovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		SystemToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return SystemToRecord.Get();
	}

private:
	/** Object to record from */
	TWeakObjectPtr<class UParticleSystemComponent> SystemToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneParticleSection> MovieSceneSection;

	bool bWasTriggered;

	EParticleKey PreviousState;

	TWeakObjectPtr<UMovieSceneParticleTrackSectionRecorder> DelegateProxy;
};
