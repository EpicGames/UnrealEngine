// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneBoolSection;
struct FGuid;

class FMovieSceneSpawnSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneSpawnSectionRecorderFactory() {}

	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const override;
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
};

class FMovieSceneSpawnSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	virtual ~FMovieSceneSpawnSectionRecorder() {}

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* MovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

private:
	/** Object to record from */
	TWeakObjectPtr<UObject> ObjectToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneBoolSection> MovieSceneSection;

	bool bWasSpawned;
};
