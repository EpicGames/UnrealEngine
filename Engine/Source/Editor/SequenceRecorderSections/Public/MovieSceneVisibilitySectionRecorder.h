// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorRecordingSettings.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "MovieSceneVisibilitySectionRecorderSettings.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneBoolSection;
struct FGuid;

class FMovieSceneVisibilitySectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneVisibilitySectionRecorderFactory() {}

	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const override;
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	virtual UObject* CreateSettingsObject(class UObject* InOuter) const override { return  NewObject<UMovieSceneVisibilitySectionRecorderSettings>(InOuter, FName(TEXT("MovieSceneVisibilitySectionRecorder"))); }
};

class FMovieSceneVisibilitySectionRecorder : public IMovieSceneSectionRecorder
{
public:
	virtual ~FMovieSceneVisibilitySectionRecorder() {}

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* MovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

private:
	/** Object to record from */
	TWeakObjectPtr<class UObject> ObjectToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneBoolSection> MovieSceneSection;

	/** Flag used to track visibility state and add keys when this changes */
	bool bWasVisible;
};
