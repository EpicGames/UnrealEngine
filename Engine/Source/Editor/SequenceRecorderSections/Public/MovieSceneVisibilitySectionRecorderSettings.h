// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MovieSceneVisibilitySectionRecorderSettings.generated.h"

UCLASS()
class UMovieSceneVisibilitySectionRecorderSettings : public UObject
{
public:
	GENERATED_BODY()

	UMovieSceneVisibilitySectionRecorderSettings()
		: bRecordVisibility(true)
	{}

	/** Whether to record actor visibility. */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	bool bRecordVisibility;
};

